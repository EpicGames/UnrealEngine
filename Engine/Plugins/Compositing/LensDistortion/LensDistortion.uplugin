{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Lens Distortion (Deprecated)",
	"Description" : "This plugin has been deprecated and will be removed in a future engine version. Please update your project to use the features of the CameraCalibration plugin instead.\nPlugin to generate UV displacement for lens distortion/undistortion on the GPU from standard camera model.",
	"Category" : "Compositing",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : true,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name": "LensDistortion",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		}
	]
}