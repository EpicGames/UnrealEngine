// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LensDistortionUVGeneration.usf: Generate lens distortion and undistortion
	UV displacement map into a render target.

	The pixel shader directly compute the distort viewport UV to undistort
	viewport UV displacment using Sv_Position and the reference equations and
	store them into the red and green channels.

	However to avoid resolving with a ferrari method, or doing a newton method
	on the GPU to compute the undistort viewport UV to distort viewport UV
	displacement, this couple of shaders works as follow: The vertex shader
	undistort the grid's vertices, and passdown to the pixel shader the viewport
	UV of where they should have been on screen without undistortion. The pixel
	shader can then generate the undistort viewport UV to distort viewport UV
	displacement by just subtracting the pixel's viewport UV.
=============================================================================*/

#include "/Engine/Public/Platform.ush"


// Size of the pixels in the viewport UV coordinates.
float2 PixelUVSize;

// K1, K2, K3
float3 RadialDistortionCoefs;

// P1, P2
float2 TangentialDistortionCoefs;

// Camera matrix of the undistorted viewport.
float4 UndistortedCameraMatrix;

// Camera matrix of the distorted viewport.
float4 DistortedCameraMatrix;

// Output multiply and add to the render target.
float2 OutputMultiplyAndAdd;


// Undistort a view position at V.z=1.
float2 UndistortNormalizedViewPosition(float2 V)
{
	float2 V2 = V * V;
	float R2 = V2.x + V2.y;

	// Radial distortion (extra parenthesis to match MF_Undistortion.uasset).
	float2 UndistortedV = V * (1.0 + R2 * (RadialDistortionCoefs.x + R2 * (RadialDistortionCoefs.y + R2 * RadialDistortionCoefs.z)));

	// Tangential distortion.
	UndistortedV.x += TangentialDistortionCoefs.y * (R2 + 2 * V2.x) + 2 * TangentialDistortionCoefs.x * V.x * V.y;
	UndistortedV.y += TangentialDistortionCoefs.x * (R2 + 2 * V2.y) + 2 * TangentialDistortionCoefs.y * V.x * V.y;

	return UndistortedV;
}


// Returns the undistorted viewport UV of the distorted's viewport UV.
//
// Notes:
//		UVs are bottom left originated.
float2 UndistortViewportUV(float2 ViewportUV)
{
	// Distorted viewport UV -> Distorted view position (z=1)
	float2 DistortedViewPosition = (ViewportUV - DistortedCameraMatrix.zw) / DistortedCameraMatrix.xy;

	// Compute undistorted view position (z=1)
	float2 UndistortedViewPosition = UndistortNormalizedViewPosition(DistortedViewPosition);

	// Undistorted view position (z=1) -> Undistorted viewport UV.
	return UndistortedCameraMatrix.xy * UndistortedViewPosition + UndistortedCameraMatrix.zw;
}


// Flip UV's y component.
float2 FlipUV(float2 UV)
{
	return float2(UV.x, 1 - UV.y);
}


void MainVS(
	in uint GlobalVertexId : SV_VertexID,
	out float2 OutVertexDistortedViewportUV : TEXCOORD0,
	out float4 OutPosition : SV_POSITION
	)
{
	// Compute the cell index.
	uint GridCellIndex = GlobalVertexId / 6;

	// Compute row and column id of the cell within the grid.
	uint GridColumnId = GridCellIndex / GRID_SUBDIVISION_Y;
	uint GridRowId = GridCellIndex - GridColumnId * GRID_SUBDIVISION_Y;

	// Compute the vertex id within a 2 triangles grid cell.
	uint VertexId = GlobalVertexId - GridCellIndex * 6;

	// Compute the bottom left originated UV coordinate of the triangle's vertex within the cell.
	float2 CellVertexUV = float2(0x1 & ((VertexId + 1) / 3), VertexId & 0x1);

	// Compute the top left originated UV of the vertex within the grid.
	float2 GridInvSize = 1.f / float2(GRID_SUBDIVISION_X, GRID_SUBDIVISION_Y);
	float2 GridVertexUV = FlipUV(
		GridInvSize * (CellVertexUV + float2(GridColumnId, GridRowId)));

	// The standard doesn't have half pixel shift.
	GridVertexUV -= PixelUVSize * 0.5;

	// Output vertex position.
	OutPosition = float4(FlipUV(
		UndistortViewportUV(GridVertexUV) + PixelUVSize * 0.5) * 2 - 1, 0, 1);

	// Output top left originated UV of the vertex.
	OutVertexDistortedViewportUV = GridVertexUV;
}


void MainPS(
	in noperspective float2 VertexDistortedViewportUV : TEXCOORD0,
	in float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0
	)
{
	// Compute the pixel's top left originated UV.
	float2 ViewportUV = SvPosition.xy * PixelUVSize;

	// The standard doesn't have half pixel shift.
	ViewportUV -= PixelUVSize * 0.5;

	float2 DistortUVtoUndistortUV = (UndistortViewportUV((ViewportUV))) - ViewportUV;
	float2 UndistortUVtoDistortUV = VertexDistortedViewportUV - ViewportUV;

	// Output displacement channels.
	OutColor = OutputMultiplyAndAdd.y + OutputMultiplyAndAdd.x * float4(
		DistortUVtoUndistortUV, UndistortUVtoDistortUV);
}
