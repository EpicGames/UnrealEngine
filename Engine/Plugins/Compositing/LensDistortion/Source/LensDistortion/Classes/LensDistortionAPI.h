// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LensDistortionAPI.generated.h"


/** Mathematic camera model for lens distortion/undistortion.
 *
 * Camera matrix =
 *  | F.X  0  C.x |
 *  |  0  F.Y C.Y |
 *  |  0   0   1  |
 */
struct UE_DEPRECATED(5.0, "The LensDistortion plugin is deprecated. Please update your project to use the features of the CameraCalibration plugin.") FLensDistortionCameraModel;
USTRUCT(BlueprintType)
struct LENSDISTORTION_API FLensDistortionCameraModel
{
	GENERATED_USTRUCT_BODY()
	FLensDistortionCameraModel()
	{
		K1 = K2 = K3 = P1 = P2 = 0.f;
		F = FVector2D(1.f, 1.f);
		C = FVector2D(0.5f, 0.5f);
	}


	/** Radial parameter #1. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Lens Distortion|Camera Model")
	float K1;

	/** Radial parameter #2. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Lens Distortion|Camera Model")
	float K2;

	/** Radial parameter #3. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Lens Distortion|Camera Model")
	float K3;

	/** Tangential parameter #1. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Lens Distortion|Camera Model")
	float P1;

	/** Tangential parameter #2. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Lens Distortion|Camera Model")
	float P2;

	/** Camera matrix's Fx and Fy. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Lens Distortion|Camera Model")
	FVector2D F;

	/** Camera matrix's Cx and Cy. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Lens Distortion|Camera Model")
	FVector2D C;

	
	/** Undistorts 3d vector (x, y, z=1.f) in the view space and returns (x', y', z'=1.f). */
	FVector2D UndistortNormalizedViewPosition(FVector2D V) const;


	/** Returns the overscan factor required for the undistort rendering to avoid unrendered distorted pixels. */
	float GetUndistortOverscanFactor(
		float DistortedHorizontalFOV,
		float DistortedAspectRatio) const;


	/** Draws UV displacement map within the output render target.
	 * - Red & green channels hold the distortion displacement;
	 * - Blue & alpha channels hold the undistortion displacement.
	 * @param World Current world to get the rendering settings from (such as feature level).
	 * @param DistortedHorizontalFOV The desired horizontal FOV in the distorted render.
	 * @param DistortedAspectRatio The desired aspect ratio of the distorted render.
	 * @param UndistortOverscanFactor The factor of the overscan for the undistorted render.
	 * @param OutputRenderTarget The render target to draw to. Don't necessarily need to have same resolution or aspect ratio as distorted render.
	 * @param OutputMultiply The multiplication factor applied on the displacement.
	 * @param OutputAdd Value added to the multiplied displacement before storing into the output render target.
	 */
	void DrawUVDisplacementToRenderTarget(
		class UWorld* World,
		float DistortedHorizontalFOV,
		float DistortedAspectRatio,
		float UndistortOverscanFactor,
		class UTextureRenderTarget2D* OutputRenderTarget,
		float OutputMultiply,
		float OutputAdd) const;


	/** Compare two lens distortion models and return whether they are equal. */
	bool operator == (const FLensDistortionCameraModel& Other) const
	{
		return (
			K1 == Other.K1 &&
			K2 == Other.K2 &&
			K3 == Other.K3 &&
			P1 == Other.P1 &&
			P2 == Other.P2 &&
			F == Other.F &&
			C == Other.C);
	}

	/** Compare two lens distortion models and return whether they are different. */
	bool operator != (const FLensDistortionCameraModel& Other) const
	{
		return !(*this == Other);
	}
};
