// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "LensDistortionAPI.h"
#include "LensDistortionBlueprintLibrary.generated.h"


class UE_DEPRECATED(5.0, "The LensDistortion plugin is deprecated. Please update your project to use the features of the CameraCalibration plugin.") ULensDistortionBlueprintLibrary;
UCLASS(MinimalAPI, meta=(DeprecationMessage = "The LensDistortion plugin is deprecated. Please update your project to use the features of the CameraCalibration plugin.", ScriptName="LensDistortionLibrary"))
class ULensDistortionBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()


	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** Returns the overscan factor required for the undistort rendering to avoid unrendered distorted pixels. */
	UFUNCTION(BlueprintPure, Category = "Lens Distortion", meta = (DeprecatedFunction, DeprecationMessage = "The LensDistortion plugin is deprecated. Please update your project to use the features of the CameraCalibration plugin."))
	static void GetUndistortOverscanFactor(
		const FLensDistortionCameraModel& CameraModel,
		float DistortedHorizontalFOV,
		float DistortedAspectRatio,
		float& UndistortOverscanFactor);
		
	/** Draws UV displacement map within the output render target.
	 * - Red & green channels hold the distortion displacement;
	 * - Blue & alpha channels hold the undistortion displacement.
	 * @param DistortedHorizontalFOV The desired horizontal FOV in the distorted render.
	 * @param DistortedAspectRatio The desired aspect ratio of the distorted render.
	 * @param UndistortOverscanFactor The factor of the overscan for the undistorted render.
	 * @param OutputRenderTarget The render target to draw to. Don't necessarily need to have same resolution or aspect ratio as distorted render.
	 * @param OutputMultiply The multiplication factor applied on the displacement.
	 * @param OutputAdd Value added to the multiplied displacement before storing into the output render target.
	 */
	UFUNCTION(BlueprintCallable, Category = "Lens Distortion", meta = (DeprecatedFunction, DeprecationMessage = "The LensDistortion plugin is deprecated. Please update your project to use the features of the CameraCalibration plugin.", WorldContext = "WorldContextObject"))
	static void DrawUVDisplacementToRenderTarget(
		const UObject* WorldContextObject,
		const FLensDistortionCameraModel& CameraModel,
		float DistortedHorizontalFOV,
		float DistortedAspectRatio,
		float UndistortOverscanFactor,
		class UTextureRenderTarget2D* OutputRenderTarget,
		float OutputMultiply = 0.5,
		float OutputAdd = 0.5
		);

	/* Returns true if A is equal to B (A == B) */
	UFUNCTION(BlueprintPure, meta=(DeprecatedFunction, DeprecationMessage = "The LensDistortion plugin is deprecated. Please update your project to use the features of the CameraCalibration plugin.", DisplayName = "Equal (LensDistortionCameraModel)", CompactNodeTitle = "==", Keywords = "== equal"), Category = "Lens Distortion")
	static bool EqualEqual_CompareLensDistortionModels(
		const FLensDistortionCameraModel& A,
		const FLensDistortionCameraModel& B)
	{
		return A == B;
	}

	/* Returns true if A is not equal to B (A != B) */
	UFUNCTION(BlueprintPure, meta = (DeprecatedFunction, DeprecationMessage = "The LensDistortion plugin is deprecated. Please update your project to use the features of the CameraCalibration plugin.", DisplayName = "NotEqual (LensDistortionCameraModel)", CompactNodeTitle = "!=", Keywords = "!= not equal"), Category = "Lens Distortion")
	static bool NotEqual_CompareLensDistortionModels(
		const FLensDistortionCameraModel& A,
		const FLensDistortionCameraModel& B)
	{
		return A != B;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

};
