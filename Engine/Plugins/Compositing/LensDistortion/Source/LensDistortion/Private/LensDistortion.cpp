// Copyright Epic Games, Inc. All Rights Reserved.

#include "ILensDistortion.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "ShaderCore.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
class FLensDistortion : public ILensDistortion
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE( FLensDistortion, LensDistortion )


void FLensDistortion::StartupModule()
{
	FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("LensDistortion"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/LensDistortion"), PluginShaderDir);
}


void FLensDistortion::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
