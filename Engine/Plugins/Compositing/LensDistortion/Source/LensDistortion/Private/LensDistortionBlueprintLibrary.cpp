// Copyright Epic Games, Inc. All Rights Reserved.

#include "LensDistortionBlueprintLibrary.h"


PRAGMA_DISABLE_DEPRECATION_WARNINGS
ULensDistortionBlueprintLibrary::ULensDistortionBlueprintLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{ }


// static
void ULensDistortionBlueprintLibrary::GetUndistortOverscanFactor(
	const FLensDistortionCameraModel& CameraModel,
	float DistortedHorizontalFOV,
	float DistortedAspectRatio,
	float& UndistortOverscanFactor)
{
	UndistortOverscanFactor = CameraModel.GetUndistortOverscanFactor(DistortedHorizontalFOV, DistortedAspectRatio);
}


// static
void ULensDistortionBlueprintLibrary::DrawUVDisplacementToRenderTarget(
	const UObject* WorldContextObject,
	const FLensDistortionCameraModel& CameraModel,
	float DistortedHorizontalFOV,
	float DistortedAspectRatio,
	float UndistortOverscanFactor,
	class UTextureRenderTarget2D* OutputRenderTarget,
	float OutputMultiply,
	float OutputAdd)
{
	CameraModel.DrawUVDisplacementToRenderTarget(
		WorldContextObject->GetWorld(),
		DistortedHorizontalFOV, DistortedAspectRatio,
		UndistortOverscanFactor, OutputRenderTarget,
		OutputMultiply, OutputAdd);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
