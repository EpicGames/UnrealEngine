// Copyright Epic Games, Inc. All Rights Reserved.

#include "LensDistortionAPI.h"


#include "Engine/TextureRenderTarget2D.h"
#include "Engine/World.h"
#include "GlobalShader.h"
#include "PipelineStateCache.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "RHIStaticStates.h"
#include "SceneInterface.h"
#include "ShaderParameterUtils.h"
#include "Logging/MessageLog.h"
#include "TextureResource.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RenderingThread.h"


static const uint32 kGridSubdivisionX = 32;
static const uint32 kGridSubdivisionY = 16;

#define LOCTEXT_NAMESPACE "LensDistortionPlugin"


/**
 * Internal intermediary structure derived from FLensDistortionCameraModel by the game thread
 * to hand to the render thread.
 */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
struct FCompiledCameraModel
{
	/** Orignal camera model that has generated this compiled model. */
	FLensDistortionCameraModel OriginalCameraModel;

	/** Camera matrices of the lens distortion for the undistorted and distorted render.
	 *  XY holds the scales factors, ZW holds the translates.
	 */
	FVector4 DistortedCameraMatrix;
	FVector4 UndistortedCameraMatrix;

	/** Output multiply and add of the channel to the render target. */
	FVector2D OutputMultiplyAndAdd;
};


/** Undistorts top left originated viewport UV into the view space (x', y', z'=1.f) */
static FVector2D LensUndistortViewportUVIntoViewSpace(
	const FLensDistortionCameraModel& CameraModel,
	float TanHalfDistortedHorizontalFOV, float DistortedAspectRatio,
	FVector2D DistortedViewportUV)
{
	FVector2D AspectRatioAwareF = CameraModel.F * FVector2D(1, -DistortedAspectRatio);
	return CameraModel.UndistortNormalizedViewPosition((DistortedViewportUV - CameraModel.C) / AspectRatioAwareF);
}


class FLensDistortionUVGenerationShader : public FGlobalShader
{
	DECLARE_INLINE_TYPE_LAYOUT(FLensDistortionUVGenerationShader, NonVirtual);
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("GRID_SUBDIVISION_X"), kGridSubdivisionX);
		OutEnvironment.SetDefine(TEXT("GRID_SUBDIVISION_Y"), kGridSubdivisionY);
	}

	FLensDistortionUVGenerationShader() {}

	FLensDistortionUVGenerationShader(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		PixelUVSize.Bind(Initializer.ParameterMap, TEXT("PixelUVSize"));
		RadialDistortionCoefs.Bind(Initializer.ParameterMap, TEXT("RadialDistortionCoefs"));
		TangentialDistortionCoefs.Bind(Initializer.ParameterMap, TEXT("TangentialDistortionCoefs"));
		DistortedCameraMatrix.Bind(Initializer.ParameterMap, TEXT("DistortedCameraMatrix"));
		UndistortedCameraMatrix.Bind(Initializer.ParameterMap, TEXT("UndistortedCameraMatrix"));
		OutputMultiplyAndAdd.Bind(Initializer.ParameterMap, TEXT("OutputMultiplyAndAdd"));
	}

	void SetParameters(
		FRHIBatchedShaderParameters& BatchedParameters,
		const FCompiledCameraModel& CompiledCameraModel,
		const FIntPoint& DisplacementMapResolution)
	{
		FVector2f PixelUVSizeValue(
			1.f / float(DisplacementMapResolution.X), 1.f / float(DisplacementMapResolution.Y));
		FVector3f RadialDistortionCoefsValue(
			CompiledCameraModel.OriginalCameraModel.K1,
			CompiledCameraModel.OriginalCameraModel.K2,
			CompiledCameraModel.OriginalCameraModel.K3);
		FVector2f TangentialDistortionCoefsValue(
			CompiledCameraModel.OriginalCameraModel.P1,
			CompiledCameraModel.OriginalCameraModel.P2);

		SetShaderValue(BatchedParameters, PixelUVSize, PixelUVSizeValue);
		SetShaderValue(BatchedParameters, DistortedCameraMatrix, FVector4f(CompiledCameraModel.DistortedCameraMatrix));
		SetShaderValue(BatchedParameters, UndistortedCameraMatrix, FVector4f(CompiledCameraModel.UndistortedCameraMatrix));
		SetShaderValue(BatchedParameters, RadialDistortionCoefs, RadialDistortionCoefsValue);
		SetShaderValue(BatchedParameters, TangentialDistortionCoefs, TangentialDistortionCoefsValue);
		SetShaderValue(BatchedParameters, OutputMultiplyAndAdd, FVector2f(CompiledCameraModel.OutputMultiplyAndAdd));
	}

private:
	
	LAYOUT_FIELD(FShaderParameter, PixelUVSize);
	LAYOUT_FIELD(FShaderParameter, RadialDistortionCoefs);
	LAYOUT_FIELD(FShaderParameter, TangentialDistortionCoefs);
	LAYOUT_FIELD(FShaderParameter, DistortedCameraMatrix);
	LAYOUT_FIELD(FShaderParameter, UndistortedCameraMatrix);
	LAYOUT_FIELD(FShaderParameter, OutputMultiplyAndAdd);
};


class FLensDistortionUVGenerationVS : public FLensDistortionUVGenerationShader
{
	DECLARE_SHADER_TYPE(FLensDistortionUVGenerationVS, Global);
public:

	/** Default constructor. */
	FLensDistortionUVGenerationVS() {}

	/** Initialization constructor. */
	FLensDistortionUVGenerationVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FLensDistortionUVGenerationShader(Initializer)
	{
	}
};


class FLensDistortionUVGenerationPS : public FLensDistortionUVGenerationShader
{
	DECLARE_SHADER_TYPE(FLensDistortionUVGenerationPS, Global);
public:

	/** Default constructor. */
	FLensDistortionUVGenerationPS() {}

	/** Initialization constructor. */
	FLensDistortionUVGenerationPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FLensDistortionUVGenerationShader(Initializer)
	{ }
};


IMPLEMENT_SHADER_TYPE(, FLensDistortionUVGenerationVS, TEXT("/Plugin/LensDistortion/Private/UVGeneration.usf"), TEXT("MainVS"), SF_Vertex)
IMPLEMENT_SHADER_TYPE(, FLensDistortionUVGenerationPS, TEXT("/Plugin/LensDistortion/Private/UVGeneration.usf"), TEXT("MainPS"), SF_Pixel)


static void DrawUVDisplacementToRenderTarget_RenderThread(
	FRHICommandListImmediate& RHICmdList,
	const FCompiledCameraModel& CompiledCameraModel,
	const FName& TextureRenderTargetName,
	FTextureRenderTargetResource* OutTextureRenderTargetResource,
	ERHIFeatureLevel::Type FeatureLevel)
{
	check(IsInRenderingThread());

	SCOPED_DRAW_EVENTF(RHICmdList, SceneCapture, TEXT("LensDistortionDisplacementGeneration %s"), TextureRenderTargetName);

	FRHITexture* RenderTargetTexture = OutTextureRenderTargetResource->GetRenderTargetTexture();

	RHICmdList.Transition(FRHITransitionInfo(RenderTargetTexture, ERHIAccess::SRVMask, ERHIAccess::RTV));

	FRHIRenderPassInfo RPInfo(RenderTargetTexture, ERenderTargetActions::DontLoad_Store);
	RHICmdList.BeginRenderPass(RPInfo, TEXT("DrawUVDisplacement"));
	{
		FIntPoint DisplacementMapResolution(OutTextureRenderTargetResource->GetSizeX(), OutTextureRenderTargetResource->GetSizeY());

		// Update viewport.
		RHICmdList.SetViewport(
			0, 0, 0.f,
			DisplacementMapResolution.X, DisplacementMapResolution.Y, 1.f);

		// Get shaders.
		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(FeatureLevel);
		TShaderMapRef< FLensDistortionUVGenerationVS > VertexShader(GlobalShaderMap);
		TShaderMapRef< FLensDistortionUVGenerationPS > PixelShader(GlobalShaderMap);

		// Set the graphic pipeline state.
		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		// Update viewport.
		RHICmdList.SetViewport(
			0, 0, 0.f,
			OutTextureRenderTargetResource->GetSizeX(), OutTextureRenderTargetResource->GetSizeY(), 1.f);

		// Update shader uniform parameters.

		SetShaderParametersLegacyVS(RHICmdList, VertexShader, CompiledCameraModel, DisplacementMapResolution);
		SetShaderParametersLegacyPS(RHICmdList, PixelShader, CompiledCameraModel, DisplacementMapResolution);

		// Draw grid.
		uint32 PrimitiveCount = kGridSubdivisionX * kGridSubdivisionY * 2;
		RHICmdList.DrawPrimitive(0, PrimitiveCount, 1);
	}
	RHICmdList.EndRenderPass();

	RHICmdList.Transition(FRHITransitionInfo(RenderTargetTexture, ERHIAccess::RTV, ERHIAccess::SRVMask));
}


FVector2D FLensDistortionCameraModel::UndistortNormalizedViewPosition(FVector2D EngineV) const
{
	// Engine view space -> standard view space.
	FVector2D V = FVector2D(1, -1) * EngineV;

	FVector2D V2 = V * V;
	float R2 = V2.X + V2.Y;

	// Radial distortion (extra parenthesis to match MF_Undistortion.uasset).
	FVector2D UndistortedV = V * (1.0 + (R2 * K1 + (R2 * R2) * K2 + (R2 * R2 * R2) * K3));

	// Tangential distortion.
	UndistortedV.X += P2 * (R2 + 2 * V2.X) + 2 * P1 * V.X * V.Y;
	UndistortedV.Y += P1 * (R2 + 2 * V2.Y) + 2 * P2 * V.X * V.Y;

	// Returns engine V.
	return UndistortedV * FVector2D(1, -1);
}


/** Compiles the camera model. */
float FLensDistortionCameraModel::GetUndistortOverscanFactor(
	float DistortedHorizontalFOV, float DistortedAspectRatio) const
{
	// If the lens distortion model is identity, then early return 1.
	if (*this == FLensDistortionCameraModel())
	{
		return 1.0f;
	}

	float TanHalfDistortedHorizontalFOV = FMath::Tan(DistortedHorizontalFOV * 0.5f);

	// Get the position in the view space at z'=1 of different key point in the distorted Viewport UV coordinate system.
	// This very approximative to know the required overscan scale factor of the undistorted viewport, but works really well in practice.
	//
	//  Undistorted UV position in the view space:
	//                 ^ View space's Y
	//                 |
	//        0        1        2
	//     
	//        7        0        3 --> View space's X
	//     
	//        6        5        4
	FVector2D UndistortCornerPos0 = LensUndistortViewportUVIntoViewSpace(
		*this, TanHalfDistortedHorizontalFOV, DistortedAspectRatio, FVector2D(0.0f, 0.0f));
	FVector2D UndistortCornerPos1 = LensUndistortViewportUVIntoViewSpace(
		*this, TanHalfDistortedHorizontalFOV, DistortedAspectRatio, FVector2D(0.5f, 0.0f));
	FVector2D UndistortCornerPos2 = LensUndistortViewportUVIntoViewSpace(
		*this, TanHalfDistortedHorizontalFOV, DistortedAspectRatio, FVector2D(1.0f, 0.0f));
	FVector2D UndistortCornerPos3 = LensUndistortViewportUVIntoViewSpace(
		*this, TanHalfDistortedHorizontalFOV, DistortedAspectRatio, FVector2D(1.0f, 0.5f));
	FVector2D UndistortCornerPos4 = LensUndistortViewportUVIntoViewSpace(
		*this, TanHalfDistortedHorizontalFOV, DistortedAspectRatio, FVector2D(1.0f, 1.0f));
	FVector2D UndistortCornerPos5 = LensUndistortViewportUVIntoViewSpace(
		*this, TanHalfDistortedHorizontalFOV, DistortedAspectRatio, FVector2D(0.5f, 1.0f));
	FVector2D UndistortCornerPos6 = LensUndistortViewportUVIntoViewSpace(
		*this, TanHalfDistortedHorizontalFOV, DistortedAspectRatio, FVector2D(0.0f, 1.0f));
	FVector2D UndistortCornerPos7 = LensUndistortViewportUVIntoViewSpace(
		*this, TanHalfDistortedHorizontalFOV, DistortedAspectRatio, FVector2D(0.0f, 0.5f));

	// Find min and max of the inner square of undistorted Viewport in the view space at z'=1.
	FVector2D MinInnerViewportRect;
	FVector2D MaxInnerViewportRect;
	MinInnerViewportRect.X = FMath::Max3(UndistortCornerPos0.X, UndistortCornerPos6.X, UndistortCornerPos7.X);
	MinInnerViewportRect.Y = FMath::Max3(UndistortCornerPos4.Y, UndistortCornerPos5.Y, UndistortCornerPos6.Y);
	MaxInnerViewportRect.X = FMath::Min3(UndistortCornerPos2.X, UndistortCornerPos3.X, UndistortCornerPos4.X);
	MaxInnerViewportRect.Y = FMath::Min3(UndistortCornerPos0.Y, UndistortCornerPos1.Y, UndistortCornerPos2.Y);

	check(MinInnerViewportRect.X < 0.f);
	check(MinInnerViewportRect.Y < 0.f);
	check(MaxInnerViewportRect.X > 0.f);
	check(MaxInnerViewportRect.Y > 0.f);

	// Compute tan(VerticalFOV * 0.5)
	float TanHalfDistortedVerticalFOV = TanHalfDistortedHorizontalFOV / DistortedAspectRatio;

	// Compute the required undistorted viewport scale on each axes.
	FVector2D ViewportScaleUpFactorPerViewAxis = 0.5 * FVector2D(
		TanHalfDistortedHorizontalFOV / FMath::Max(-MinInnerViewportRect.X, MaxInnerViewportRect.X),
		TanHalfDistortedVerticalFOV / FMath::Max(-MinInnerViewportRect.Y, MaxInnerViewportRect.Y));

	// Scale up by 2% more the undistorted viewport size in the view space to work
	// around the fact that odd undistorted positions might not exactly be at the minimal
	// in case of a tangential distorted barrel lens distortion.
	const float ViewportScaleUpConstMultiplier = 1.02f;
	return FMath::Max(ViewportScaleUpFactorPerViewAxis.X, ViewportScaleUpFactorPerViewAxis.Y) * ViewportScaleUpConstMultiplier;
}


void FLensDistortionCameraModel::DrawUVDisplacementToRenderTarget(
	UWorld* World,
	float DistortedHorizontalFOV,
	float DistortedAspectRatio,
	float UndistortOverscanFactor,
	UTextureRenderTarget2D* OutputRenderTarget,
	float OutputMultiply,
	float OutputAdd) const
{
	check(IsInGameThread());

	if (!OutputRenderTarget)
	{
		FMessageLog("Blueprint").Warning(LOCTEXT("LensDistortionCameraModel_OutputTargetRequired", "DrawUVDisplacementToRenderTarget: Output render target is required."));
		return;
	}

	// Compiles the camera model to know the overscan scale factor.
	float TanHalfUndistortedHorizontalFOV = FMath::Tan(DistortedHorizontalFOV * 0.5f) * UndistortOverscanFactor;
	float TanHalfUndistortedVerticalFOV = TanHalfUndistortedHorizontalFOV / DistortedAspectRatio;

	// Output.
	FCompiledCameraModel CompiledCameraModel;
	CompiledCameraModel.OriginalCameraModel = *this;

	CompiledCameraModel.DistortedCameraMatrix.X = 1.0f / TanHalfUndistortedHorizontalFOV;
	CompiledCameraModel.DistortedCameraMatrix.Y = 1.0f / TanHalfUndistortedVerticalFOV;
	CompiledCameraModel.DistortedCameraMatrix.Z = 0.5f;
	CompiledCameraModel.DistortedCameraMatrix.W = 0.5f;

	CompiledCameraModel.UndistortedCameraMatrix.X = F.X;
	CompiledCameraModel.UndistortedCameraMatrix.Y = F.Y * DistortedAspectRatio;
	CompiledCameraModel.UndistortedCameraMatrix.Z = C.X;
	CompiledCameraModel.UndistortedCameraMatrix.W = C.Y;

	CompiledCameraModel.OutputMultiplyAndAdd.X = OutputMultiply;
	CompiledCameraModel.OutputMultiplyAndAdd.Y = OutputAdd;

	const FName TextureRenderTargetName = OutputRenderTarget->GetFName();
	FTextureRenderTargetResource* TextureRenderTargetResource = OutputRenderTarget->GameThread_GetRenderTargetResource();

	ERHIFeatureLevel::Type FeatureLevel = World->Scene->GetFeatureLevel();

	if (FeatureLevel < ERHIFeatureLevel::SM5)
	{
		FMessageLog("Blueprint").Warning(LOCTEXT("LensDistortionCameraModel_SM5Unavailable", "DrawUVDisplacementToRenderTarget: Requires RHIFeatureLevel::SM5 which is unavailable."));
		return;
	}

	ENQUEUE_RENDER_COMMAND(CaptureCommand)(
		[CompiledCameraModel, TextureRenderTargetResource, TextureRenderTargetName, FeatureLevel](FRHICommandListImmediate& RHICmdList)
		{
			DrawUVDisplacementToRenderTarget_RenderThread(
				RHICmdList,
				CompiledCameraModel,
				TextureRenderTargetName,
				TextureRenderTargetResource,
				FeatureLevel);
		}
	);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#undef LOCTEXT_NAMESPACE
